{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "0.1",
	"FriendlyName": "AutomatedPerfTestTools",
	"Description": "Tools for Automated Perf Testing framework",
	"Category": "Testing",
	"CreatedBy": "",
	"CreatedByURL": "",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": true,
	"IsBetaVersion": false,
	"IsExperimentalVersion": true,
	"Installed": false,
	"EnabledByDefault": true,
	"Modules": [
		{
			"Name": "AutomatedPerfTestLaunchExtension",
			"Type": "Editor",
			"LoadingPhase": "Default",
			"PlatformAllowList": [
				"Win64",
				"Linux",
				"Mac"
			],
			"ProgramAllowList": [ "UnrealFrontend" ]
		}
	],
	"SupportedPrograms": [ "UnrealFrontend" ],
	"Plugins": [
		{
			"Name": "ProjectLauncher",
			"Enabled": true,
			"SupportedTargetPlatforms": [
				"Win64",
				"Linux",
				"Mac"
			]
		}
	]
}