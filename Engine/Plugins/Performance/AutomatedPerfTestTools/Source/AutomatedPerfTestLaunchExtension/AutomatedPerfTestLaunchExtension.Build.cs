// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class AutomatedPerfTestLaunchExtension : ModuleRules
{
	public AutomatedPerfTestLaunchExtension(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.AddRange(
		[
			"Core",
			"CoreUObject",
			"Json",
			"Slate",
			"SlateCore",
			"Sockets",
			"TraceLog",
			"ProjectLauncher",
			"DeveloperSettings",
			"ToolWidgets",
		]);


		if (Target.bBuildEditor)
		{
			PrivateDependencyModuleNames.AddRange(
			[
				"UnrealEd",
				"Engine"
			]);
		}
	}
}
