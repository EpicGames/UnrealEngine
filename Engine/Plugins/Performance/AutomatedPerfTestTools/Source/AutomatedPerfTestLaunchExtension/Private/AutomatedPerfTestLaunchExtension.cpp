// Copyright Epic Games, Inc. All Rights Reserved.

#include "AutomatedPerfTestLaunchExtension.h"
#include "Templates/SharedPointer.h"
#include "Model/ProjectLauncherModel.h"
#include "Misc/Paths.h"
#include "Misc/ConfigCacheIni.h"

#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Shared/SCustomLaunchCombo.h"

#define LOCTEXT_NAMESPACE "FAutomatedPerfTestLaunchExtension"

namespace AutomatedPerfTestConfig
{
	using EConfigType = ProjectLauncher::FLaunchExtensionInstance::EConfig;
	constexpr EConfigType ConfigTypeCommon = EConfigType::PerProfile;

	constexpr const TCHAR* EnableCSVProfilerConfig = TEXT("EnableCSVProfiler");
	constexpr const TCHAR* WindowedConfig = TEXT("Windowed");
	constexpr const TCHAR* IterationsConfig = TEXT("Iterations");
	constexpr const TCHAR* LLMConfig = TEXT("LLM");
	constexpr const TCHAR* GPUPerfConfig = TEXT("GPUPerf");
	constexpr const TCHAR* GPUReshapeConfig = TEXT("GPUReshape");
	constexpr const TCHAR* LockDynamicResConfig = TEXT("LockDynamicRes");
	constexpr const TCHAR* CVarConfig = TEXT("CVars");
	constexpr const TCHAR* TestIDConfig = TEXT("TestID");
	constexpr const TCHAR* DeterministicConfig = TEXT("Deterministic");
	constexpr const TCHAR* InsightsConfig = TEXT("Insights");
	constexpr const TCHAR* InsightsTraceChannelsConfig = TEXT("InsightsTraceChannels");

	constexpr const TCHAR* ProfileGoMapConfig = TEXT("ProfileGoMapName");
	constexpr const TCHAR* ReplayConfig = TEXT("ReplayTestName");
	constexpr const TCHAR* SequenceTestNameConfig = TEXT("SequenceTestName");
	constexpr const TCHAR* StaticCameraMapConfig = TEXT("StaticCameraMapName");
	constexpr const TCHAR* ProfileGoConfigFileName = TEXT("ProfileGoConfigFile");

	constexpr EAutomatedPerfTestType DefaultTestType = EAutomatedPerfTestType::Sequence;
	constexpr uint32 DefaultIterationCount = 1;

	static constexpr bool IsEditorBuild()
	{
#if WITH_EDITOR
		constexpr bool bIsEditor = true;
#else
		constexpr bool bIsEditor = false;
#endif
		return bIsEditor;
	}
}

using namespace AutomatedPerfTestConfig;

void FAutomatedPerfTestLaunchExtensionInstance::OnPropertyChanged()
{
	ILauncherProfileAutomatedTestPtr Test = GetTest();
	if(Test.IsValid())
	{
		// This ensures the test type is always 
		// updated to the current selected one. 
		OnTestAdded(Test.ToSharedRef());
	}

	Super::OnPropertyChanged();
}

static constexpr const TCHAR* GetTestNodeName(EAutomatedPerfTestType TestType)
{
	// This is the string which represents test node defined in the
	// Gauntlet controller. The test launch will fail if this string
	// does not match the name of the controller class in Gauntlet.
	switch (TestType)
	{
	case EAutomatedPerfTestType::Sequence:		return TEXT("AutomatedPerfTest.SequenceTest");
	case EAutomatedPerfTestType::Replay:		return TEXT("AutomatedPerfTest.ReplayTest");
	case EAutomatedPerfTestType::StaticCamera:	return TEXT("AutomatedPerfTest.StaticCameraTest");
	case EAutomatedPerfTestType::Material:		return TEXT("AutomatedPerfTest.MaterialTest");
	case EAutomatedPerfTestType::ProfileGo:		return TEXT("AutomatedPerfTest.ProfileGoTest");
	default:									return TEXT("AutomatedPerfTest.DefaultTest");
	}
}

void FAutomatedPerfTestLaunchExtensionInstance::OnTestAdded(ILauncherProfileAutomatedTestRef AutomatedTest)
{
	const TCHAR* TestNode = GetTestNodeName(GetTestType());

	AutomatedTest->SetTests(TestNode);
}

const ProjectLauncher::FProjectSettings FAutomatedPerfTestLaunchExtensionInstance::GetProjectSettings() const
{
	using namespace ProjectLauncher;
	TSharedRef<FModel> LauncherModel = GetModel();
	return LauncherModel->GetProjectSettings(GetProfile());
}

const FConfigSection* FAutomatedPerfTestLaunchExtensionInstance::GetConfigSection(EAutomatedPerfTestType TestType) const
{
	const ProjectLauncher::FProjectSettings ProjectSettings = GetProjectSettings();
	FConfigCacheIni* const Config = ProjectSettings.Config;
	static const TMap<EAutomatedPerfTestType, FString> TestTypeSectionMap =
	{
		{ EAutomatedPerfTestType::Sequence,		TEXT("/Script/AutomatedPerfTesting.AutomatedSequencePerfTestProjectSettings") },
		{ EAutomatedPerfTestType::Replay,		TEXT("/Script/AutomatedPerfTesting.AutomatedReplayPerfTestProjectSetting") },
		{ EAutomatedPerfTestType::ProfileGo,	TEXT("/Script/AutomatedPerfTesting.AutomatedProfileGoTestProjectSetting") },
		{ EAutomatedPerfTestType::StaticCamera,	TEXT("/Script/AutomatedPerfTesting.AutomatedStaticCameraPerfTestProjectSettings") },
		{ EAutomatedPerfTestType::Material,		TEXT("/Script/AutomatedPerfTesting.AutomatedMaterialPerfTestProjectSetting") },
	};

	const FConfigSection* Section = Config->GetSection(*TestTypeSectionMap[TestType], true, GEngineIni);
	return Section;
}

void FAutomatedPerfTestLaunchExtensionInstance::CustomizeTree(ProjectLauncher::FLaunchProfileTreeData& ProfileTreeData)
{
	using namespace ProjectLauncher;

	const auto GetDisplayName = [](EAutomatedPerfTestType Type)
	{
		switch (Type)
		{
			case EAutomatedPerfTestType::Sequence:		return LOCTEXT("SequenceLabel", "Sequence");
			case EAutomatedPerfTestType::Replay:		return LOCTEXT("ReplayLabel", "Replay");
			case EAutomatedPerfTestType::ProfileGo:		return LOCTEXT("ProfileGoLabel", "ProfileGo");
			case EAutomatedPerfTestType::StaticCamera:	return LOCTEXT("StaticCameraLabel", "StaticCamera");
			case EAutomatedPerfTestType::Material:		return LOCTEXT("MaterialLabel", "Material");
		}
		return FText::GetEmpty();
	};

	const auto GetToolTip = [](EAutomatedPerfTestType Type)
	{
		switch (Type)
		{
		case EAutomatedPerfTestType::Sequence:		return LOCTEXT("SequenceToolTipLabel", "Automated Sequence Perf Test");
		case EAutomatedPerfTestType::Replay:		return LOCTEXT("ReplayToolTipLabel", "Automated Replay Perf Test");
		case EAutomatedPerfTestType::ProfileGo:		return LOCTEXT("ProfileGoToolTipLabel", "Automated ProfileGo Perf Test");
		case EAutomatedPerfTestType::StaticCamera:	return LOCTEXT("StaticCameraToolTipLabel", "Automated StaticCamera Perf Test");
		case EAutomatedPerfTestType::Material:		return LOCTEXT("MaterialToolTipLabel", "Automated Material Perf Test");
		}
		return FText::GetEmpty();
	};

	FLaunchProfileTreeNode& TreeNode = 
		AddDefaultHeading(ProfileTreeData)
		.AddBoolean(LOCTEXT("APTEnableCSVLabel", "Enable CSV Profiler"), 
		{
			.GetValue = [this]() { return GetConfigBool(ConfigTypeCommon, EnableCSVProfilerConfig, true); },
			.SetValue = [this](bool bEnable) { return SetConfigBool(ConfigTypeCommon, EnableCSVProfilerConfig, bEnable); },
			.GetDefaultValue = [this]() { return true; }
		})
		.AddBoolean(LOCTEXT("WindowedLabel","Windowed"),
		{
			.GetValue = [this]()          { return GetConfigBool(ConfigTypeCommon, WindowedConfig);},
			.SetValue = [this](bool bVal) { SetConfigBool(ConfigTypeCommon, WindowedConfig, bVal ); },
		})
		.AddInteger(LOCTEXT("IterationsLabel", "Iterations"), 
		{
			.GetValue = [this]() { return GetConfigInteger(ConfigTypeCommon, IterationsConfig, DefaultIterationCount); },
			.SetValue = [this](int32 Value) { SetConfigInteger(ConfigTypeCommon, IterationsConfig, Value); },
		})
		.AddWidget(LOCTEXT("TestTypeLabel", "Test Type"),
		{
			.IsDefault = [this]() { return GetTestType() == DefaultTestType; },
			.SetToDefault = [this]() { SetTestType(DefaultTestType); },
		},
		SNew(SCustomLaunchCombo<EAutomatedPerfTestType>)
		.OnSelectionChanged(this, &FAutomatedPerfTestLaunchExtensionInstance::OnTestTypeSelectionChanged)
		.SelectedItem(this, &FAutomatedPerfTestLaunchExtensionInstance::GetTestType)
		.GetDisplayName_Lambda(GetDisplayName)
		.GetItemToolTip_Lambda(GetToolTip)
		.Items(TArray<EAutomatedPerfTestType>(
			{
				EAutomatedPerfTestType::Sequence,
				EAutomatedPerfTestType::Replay ,
				EAutomatedPerfTestType::ProfileGo ,
				EAutomatedPerfTestType::StaticCamera ,
				EAutomatedPerfTestType::Material
			})
		));

	// Test type specific options will be added here. 
	FLaunchProfileTreeNode& TestTypeTreeNode = TreeNode.AddSubHeading(TEXT("TestTypeOptions"), LOCTEXT("TestTypeOptionsLabel", "Test Type Options"));
	AddTestNodeOptions(TestTypeTreeNode);

	// Add "Advanced" options 
	TreeNode
		.AddSubHeading(TEXT("Advanced"), LOCTEXT("APTAdvancedLabel", "Advanced"))
		.AddString(LOCTEXT("TestIDLabel", "Test ID"),
		{
			.GetValue = [this]() { return GetConfigString(ConfigTypeCommon, TestIDConfig); },
			.SetValue = [this](FString TestID) { return SetConfigString(ConfigTypeCommon, TestIDConfig, TestID); }
		}, 
		LOCTEXT("TestIDHint", "My-Test-Identifier"), 
		LOCTEXT("TestIDTooltip", "If you want to differentiate between runs, give this test run a name or ID"))
		.AddBoolean(LOCTEXT("LLMLabel","Enable LLM"),
		{
			.GetValue = [this]()          { return GetConfigBool(ConfigTypeCommon, LLMConfig);},
			.SetValue = [this](bool bVal) { SetConfigBool(ConfigTypeCommon, LLMConfig, bVal ); },
		},
		LOCTEXT("LLMToolTip", "Enables Low Level Memory (LLM) Tracking Stats"))
		.AddBoolean(LOCTEXT("InsightsLabel","Enable Insights"),
		{
			.GetValue = [this]()          { return GetConfigBool(ConfigTypeCommon, InsightsConfig);},
			.SetValue = [this](bool bVal) { SetConfigBool(ConfigTypeCommon, InsightsConfig, bVal ); },
		},
		LOCTEXT("InsightsToolTip", "Enables Insights Trace"))
		.AddBoolean(LOCTEXT("DeterministicLabel","Deterministic"),
		{
			.GetValue = [this]()          { return GetConfigBool(ConfigTypeCommon, DeterministicConfig);},
			.SetValue = [this](bool bVal) { SetConfigBool(ConfigTypeCommon, DeterministicConfig, bVal ); },
			.GetDefaultValue = [this]()	  { return true; }
		})
		.AddString(LOCTEXT("TraceChannelsLabel", "Trace Channels"),
		{
			.GetValue = [this]() { return GetConfigString(ConfigTypeCommon, InsightsTraceChannelsConfig); },
			.SetValue = [this](FString Channels) { return SetConfigString(ConfigTypeCommon, InsightsTraceChannelsConfig, Channels); },
			.GetDefaultValue = [this]() { return TEXT("default,screenshot,stats"); }
		}, 
		LOCTEXT("TraceChannelsHint", "default,channel1,channel2,..."),
		LOCTEXT("TraceChannelsTooltip", "Trace Channels to use if Insights Trace is enabled. Comma separated values. Default=default,screenshot,stats"))
		.AddBoolean(LOCTEXT("GPUPerfLabel","Enable GPU Perf"),
		{
			.GetValue = [this]()          { return GetConfigBool(ConfigTypeCommon, GPUPerfConfig);},
			.SetValue = [this](bool bVal) { SetConfigBool(ConfigTypeCommon, GPUPerfConfig, bVal ); },
		},

		LOCTEXT("GPUPerfHint", "Enables GPU Perf related CVars and locks dynamic resolution"))
		.AddBoolean(LOCTEXT("GPUReshapeLabel","Enable GPU Reshape"),
		{
			.GetValue = [this]()          { return GetConfigBool(ConfigTypeCommon, GPUReshapeConfig);},
			.SetValue = [this](bool bVal) { SetConfigBool(ConfigTypeCommon, GPUReshapeConfig, bVal ); },
		})
		.AddBoolean(LOCTEXT("LockDynResLabel","Lock Dynamic Resolution"),
		{
			.GetValue = [this]()          { return GetConfigBool(ConfigTypeCommon, LockDynamicResConfig);},
			.SetValue = [this](bool bVal) { SetConfigBool(ConfigTypeCommon, LockDynamicResConfig, bVal ); },
		},
		LOCTEXT("LockDynResHint", "Locks Dynamic Resolution (Default = 60%). This enabled by default if GPU Perf is enabled."))
		.AddString(LOCTEXT("CVarsLabel", "CVars"), 
		{
			.GetValue = [this]() { return GetConfigString(ConfigTypeCommon, CVarConfig); },
			.SetValue = [this](FString Cvars) { return SetConfigString(ConfigTypeCommon, CVarConfig, Cvars); }
		},
		LOCTEXT("CVarsHint", "r.Var1, r.Var2=1, ..."), 
		LOCTEXT("CVarsToolTip", "Comma separated CVars to be set before test."));
}

void FAutomatedPerfTestLaunchExtensionInstance::CustomizeUATCommandLine(FString& InOutCommandLine)
{
	// Most of the parameters here are defined and handled in the corresponding 
	// Gauntlet controller which launches the automated perf test with the required 
	// params based on the options passed down here.

	if (GetConfigBool(ConfigTypeCommon, EnableCSVProfilerConfig, true))
	{
		InOutCommandLine += TEXT(" -AutomatedPerfTest.DoCSVProfiler");
	}

	if (GetConfigBool(ConfigTypeCommon, LLMConfig))
	{
		InOutCommandLine += TEXT(" -AutomatedPerfTest.DoLLM");
	}

	if (GetConfigBool(ConfigTypeCommon, GPUPerfConfig))
	{
		InOutCommandLine += TEXT(" -AutomatedPerfTest.DoGPUPerf");
	}

	if (GetConfigBool(ConfigTypeCommon, GPUReshapeConfig))
	{
		InOutCommandLine += TEXT(" -AutomatedPerfTest.DoGPUReshape");
	}

	if (GetConfigBool(ConfigTypeCommon, LockDynamicResConfig))
	{
		InOutCommandLine += TEXT(" -AutomatedPerfTest.LockDynamicRes");
	}

	if (GetConfigBool(ConfigTypeCommon, InsightsConfig))
	{
		InOutCommandLine += TEXT(" -AutomatedPerfTest.DoInsightsTrace");
	}

	// This is enabled by default.
	if (!GetConfigBool(ConfigTypeCommon, DeterministicConfig, true))
	{
		InOutCommandLine += TEXT(" -AutomatedPerfTest.Deterministic=0");
	}

	InOutCommandLine += TEXT(" -AutomatedPerfTest.IgnoreTestBuildLogging");

	const auto AddStringParamConfig = 
	[this, &InOutCommandLine](const FString& Config, const FString& Param, const FString& DefaultVal = "")
	{
		FString ConfigVal = GetConfigString(ConfigTypeCommon, *Config, *DefaultVal);
		if (!ConfigVal.IsEmpty())
		{
			InOutCommandLine += FString::Printf(TEXT(" -%s=\"%s\""), *Param, *EscapeJsonString(ConfigVal));
		}
	};

	const int32 IterationCount = GetConfigInteger(ConfigTypeCommon, IterationsConfig, DefaultIterationCount);
	InOutCommandLine += FString::Printf(TEXT(" -iterations=%d"), IterationCount);

	const EAutomatedPerfTestType Type = GetTestType();
	switch (Type)
	{
	case EAutomatedPerfTestType::ProfileGo: 
	{
	    FString ConfigPath = GetConfigString(ConfigTypeCommon, ProfileGoConfigFileName);
		if(ConfigPath.IsEmpty())
		{
			ConfigPath = FPaths::Combine(FPaths::ProjectDir(), "Saved", "Profiling", "ProfileGo.json");
		}

		if (FPaths::FileExists(ConfigPath))
		{
			InOutCommandLine += FString::Printf(TEXT(" -profilego.config=\"%s\""), *EscapeJsonString(ConfigPath));
		}

		AddStringParamConfig(ProfileGoMapConfig, TEXT("Map"));
		break;
	}
	case EAutomatedPerfTestType::Replay:
	{
		AddStringParamConfig(ReplayConfig, TEXT("AutomatedPerfTest.ReplayPerfTest.ReplayName"));
		break;
	}
	case EAutomatedPerfTestType::Sequence:
	{
		AddStringParamConfig(SequenceTestNameConfig, TEXT("AutomatedPerfTest.SequencePerfTest.MapSequenceName"));
		break;
	}
	case EAutomatedPerfTestType::Material:
	{
		// Material test uses settings for test parameters.
		break;
	}
	case EAutomatedPerfTestType::StaticCamera:
	{
		AddStringParamConfig(StaticCameraMapConfig, TEXT("AutomatedPerfTest.StaticCameraPerfTest.MapName"));
		break;
	}
	default: break;
	}

	AddStringParamConfig(CVarConfig, TEXT("AutomatedPerfTest.CVars"));
	AddStringParamConfig(TestIDConfig, TEXT("AutomatedPerfTest.TestID"));
	AddStringParamConfig(InsightsTraceChannelsConfig, TEXT("AutomatedPerfTest.TraceChannels"));

	if (GetConfigBool(ConfigTypeCommon, WindowedConfig))
	{
		InOutCommandLine += TEXT(" -windowed");
	}
}

void FAutomatedPerfTestLaunchExtensionInstance::ExportProfileGoScenarios(const FString& Filename)
{
	// Note: This needs to exist outside this extension. Profile Go scenario creation and 
	// export should ideally be handled in-editor. Disabling this for now and to be removed 
	// once ProfileGo workflow is finalized. 
#if 0
	UWorld* World = GWorld;
	UEditorEngine* EditorEngine = Cast<UEditorEngine>(GEngine);
	if (GIsEditor && EditorEngine != nullptr && EditorEngine->PlayWorld != nullptr)
	{
		World = EditorEngine->PlayWorld.Get();
	}

	if (World)
	{
		// TODO: This needs to be standardized and/or made configurable with error handling. 
		const FString OutputPath = FPaths::Combine(FPaths::ProjectDir(), "Saved", "Profiling", Filename);

		UProfileGo& ProfileGo = UProfileGo::GetCDO();
		ProfileGo.AddScenariosInLevel(World);
		ProfileGo.SaveToJSON(OutputPath);
	}
#endif
}

void FAutomatedPerfTestLaunchExtensionInstance::AddTestNodeOptions(ProjectLauncher::FLaunchProfileTreeNode& TreeNode)
{
	AddSequenceTestNodeOptions(TreeNode);
	AddReplayTestNodeOptions(TreeNode);
	AddStaticCameraTestNodeOptions(TreeNode);
	AddMaterialTestNodeOptions(TreeNode);
	AddProfileGoTestNodeOptions(TreeNode);
}

void FAutomatedPerfTestLaunchExtensionInstance::AddSequenceTestNodeOptions(ProjectLauncher::FLaunchProfileTreeNode& TreeNode)
{
	constexpr auto TestType = EAutomatedPerfTestType::Sequence;
	ProjectLauncher::FLaunchProfileTreeNode::FCallbacks&& Callbacks =
		GetTestTypeCallbacks<TestType>();

	TArray<FString> Sequences;
	if(const FConfigSection* Section = GetConfigSection(TestType))
	{
		TArray<FString> ComboStructs;
		Section->GetArray(TEXT("MapsAndSequencesToTest"), ComboStructs);

		// The ini config has the combo name as well as other
		// metadata which is not needed to launch the test. So
		// we extract the combo name only. 
		for (const FString& ComboStruct : ComboStructs)
		{
			constexpr TCHAR ComboStr[] = TEXT("ComboName=\"");
			const int32 ComboPos = ComboStruct.Find(ComboStr);
			const int32 FirstCommaPos = ComboStruct.Find(",");
			if (ComboPos == INDEX_NONE || FirstCommaPos == INDEX_NONE)
			{
				continue;
			}

			const int32 StartIndex = ComboPos + FCString::Strlen(ComboStr);
			const int32 EndIndex = FirstCommaPos - 1;
			const int32 Length = EndIndex - StartIndex;
			const FString ComboName = ComboStruct.Mid(StartIndex, Length);
			Sequences.AddUnique(ComboName);
		}
	}

	constexpr const TCHAR* ConfigName = SequenceTestNameConfig;
	auto OnSequenceChanged = [this](FString Combo)
	{
		OnPropertyChanged();
		SetConfigString(EConfig::PerProfile, ConfigName, Combo);
	};

	auto GetSequence = [this]() { return GetConfigString(EConfig::PerProfile, ConfigName); };
	auto GetDisplayName = [this](FString Combo) -> FText { return FText::FromString(GetConfigString(EConfig::PerProfile, ConfigName, *Combo)); };

	TreeNode.AddWidget(LOCTEXT("SequenceComboNameLabel", "Sequence Combo Name"),
		MoveTemp(Callbacks),
		SNew(SCustomLaunchCombo<FString>)
		.OnSelectionChanged_Lambda(MoveTemp(OnSequenceChanged))
		.SelectedItem_Lambda(MoveTemp(GetSequence))
		.GetDisplayName_Lambda(MoveTemp(GetDisplayName))
		.Items(Sequences)
	);
}

void FAutomatedPerfTestLaunchExtensionInstance::AddReplayTestNodeOptions(ProjectLauncher::FLaunchProfileTreeNode& TreeNode)
{
	ProjectLauncher::FLaunchProfileTreeNode::FCallbacks&& Callbacks =
		GetTestTypeCallbacks<EAutomatedPerfTestType::Replay>();

	const TCHAR* FileTypeFilter = TEXT("Replay files (*.replay)|*.replay");
	constexpr const TCHAR* ConfigName = ReplayConfig;
	TreeNode.AddFileString(LOCTEXT("ReplayTestNameLabel", "Replay Name"), 
	{
		.GetValue = [this]() { return GetConfigString(EConfig::PerProfile, ConfigName); },
		.SetValue = [this](FString Value) { SetConfigString(EConfig::PerProfile, ConfigName, Value); },
		.IsVisible = Callbacks.IsVisible,
		.IsEnabled = Callbacks.IsEnabled
	}, FileTypeFilter);
}

void FAutomatedPerfTestLaunchExtensionInstance::AddProfileGoTestNodeOptions(ProjectLauncher::FLaunchProfileTreeNode& TreeNode)
{
    ProjectLauncher::FLaunchProfileTreeNode::FCallbacks&& Callbacks = 
		GetTestTypeCallbacks<EAutomatedPerfTestType::ProfileGo>();

	constexpr const TCHAR* ConfigName = ProfileGoMapConfig;
	constexpr const TCHAR* ConfigFileName = ProfileGoConfigFileName;
	constexpr const TCHAR* FileTypeFilter = TEXT("ProfileGo Config JSON (*.json)|*.json");

	constexpr bool bIncludeNonContentDirMaps = true;
	const FString ProjectPath = GetProfile()->GetProjectBasePath();
	TArray<FString> Maps = GetModel()->GetAvailableProjectMapNames(ProjectPath, bIncludeNonContentDirMaps);

	auto OnMapChanged = [this](FString Map)
	{
		OnPropertyChanged();
		SetConfigString(ConfigTypeCommon, ConfigName, Map);
	};

	auto GetMap = [this]() { return GetConfigString(ConfigTypeCommon, ConfigName); };
	auto GetDisplayName = [this](FString Map) -> FText { return FText::FromString(*Map); };

	TreeNode.AddWidget(LOCTEXT("TestExportLabel", "Export ProfileGo"),
		MoveTemp(Callbacks),
		SNew(SButton)
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		.Text(LOCTEXT("TestExportLabelButton", "Export JSON"))
		.OnClicked_Lambda([this]() -> FReply
		{
			// TEMP: Will be removed once ProfileGo scenario workflow creation is finalized.
			const TCHAR* ProfileGoFilename = TEXT("ProfileGo.json");
			ExportProfileGoScenarios(ProfileGoFilename);
			return FReply::Handled();
		})
		.IsEnabled_Lambda([this]() { return GetTestType() == EAutomatedPerfTestType::ProfileGo && IsEditorBuild(); })
	)
	.AddWidget(LOCTEXT("SettingsUpdateLabel", "Update ProfileGo Settings"),
		MoveTemp(Callbacks),
		SNew(SButton)
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		.Text(LOCTEXT("SettingsUpdateLabelButton", "Update"))
		.OnClicked_Lambda([this]() -> FReply
		{
		// This needs to exist outside this extension. Disabling for now. 
		#if 0
			// TEMP: Will be removed once ProfileGo scenario workflow creation is finalized.
			UWorld* World = GWorld;
			UEditorEngine* EditorEngine = Cast<UEditorEngine>(GEngine);
			if (GIsEditor && EditorEngine != nullptr && EditorEngine->PlayWorld != nullptr)
			{
				World = EditorEngine->PlayWorld.Get();
			}
			UProfileGo::GetCDO().AddScenariosInLevel(World);
		#endif
			return FReply::Handled();
		})
		.IsEnabled_Lambda([this]() { return GetTestType() == EAutomatedPerfTestType::ProfileGo && IsEditorBuild(); })
	)
	.AddFileString(LOCTEXT("ProfileGoConfigLabel","Config File"),
	{
		.GetValue = [this]() { return GetConfigString(EConfig::PerProfile, ConfigFileName); },
		.SetValue = [this](FString Value) { SetConfigString(EConfig::PerProfile, ConfigFileName, Value); },
		.IsVisible = Callbacks.IsVisible,
		.IsEnabled = Callbacks.IsEnabled
	}, 
	FileTypeFilter)
	.AddWidget(LOCTEXT("ProfileGoMapLabel", "Map"),
		MoveTemp(Callbacks),
		SNew(SCustomLaunchCombo<FString>)
		.OnSelectionChanged_Lambda(MoveTemp(OnMapChanged))
		.SelectedItem_Lambda(MoveTemp(GetMap))
		.GetDisplayName_Lambda(MoveTemp(GetDisplayName))
		.Items(MoveTemp(Maps))
	);
}

void FAutomatedPerfTestLaunchExtensionInstance::AddStaticCameraTestNodeOptions(ProjectLauncher::FLaunchProfileTreeNode& TreeNode)
{
	constexpr auto TestType = EAutomatedPerfTestType::StaticCamera;
	ProjectLauncher::FLaunchProfileTreeNode::FCallbacks&& Callbacks =
		GetTestTypeCallbacks<TestType>();

	constexpr const TCHAR* ConfigName = StaticCameraMapConfig;

	TArray<FString> Maps;
	if(const FConfigSection* Section = GetConfigSection(TestType))
	{
		Section->GetArray(TEXT("MapsToTest"), Maps);
	}

	auto OnMapChanged = [this](FString Map)
	{
		OnPropertyChanged();
		SetConfigString(EConfig::PerProfile, ConfigName, Map);
	};

	auto GetMap = [this]() { return GetConfigString(EConfig::PerProfile, ConfigName); };
	auto GetDisplayName = [this](FString Map) -> FText { return FText::FromString(*Map); };

	TreeNode.AddWidget(LOCTEXT("StaticCameraTestNameLabel", "Static Camera Map Name"), 
		 MoveTemp(Callbacks),
		 SNew(SCustomLaunchCombo<FString>)
		.OnSelectionChanged_Lambda(MoveTemp(OnMapChanged))
		.SelectedItem_Lambda(MoveTemp(GetMap))
		.GetDisplayName_Lambda(MoveTemp(GetDisplayName))
		.Items(MoveTemp(Maps))
	);
}

void FAutomatedPerfTestLaunchExtensionInstance::AddMaterialTestNodeOptions(ProjectLauncher::FLaunchProfileTreeNode& TreeNode)
{
	// Material test does not have specific params.
}

void FAutomatedPerfTestLaunchExtensionInstance::OnTestTypeSelectionChanged(EAutomatedPerfTestType Type)
{
	SetTestType(Type);
	OnPropertyChanged();
}

void FAutomatedPerfTestLaunchExtensionInstance::SetTestType(EAutomatedPerfTestType Type)
{
	CurrentTestType = Type;
	SetConfigInteger(EConfig::PerProfile, TEXT("TestType"), (int32)Type);
	BroadcastPropertyChanged();
}

EAutomatedPerfTestType FAutomatedPerfTestLaunchExtensionInstance::GetTestType() const
{
	return (EAutomatedPerfTestType)GetConfigInteger(EConfig::PerProfile, TEXT("TestType"), (int32)CurrentTestType);
}

TSharedPtr<ProjectLauncher::FLaunchExtensionInstance> FAutomatedPerfTestLaunchExtension::CreateInstanceForProfile(ProjectLauncher::FLaunchExtensionInstance::FArgs& InArgs)
{
	return MakeShared<FAutomatedPerfTestLaunchExtensionInstance>(InArgs);
}

const TCHAR* FAutomatedPerfTestLaunchExtension::GetInternalName() const
{
	return TEXT("AutomatedPerfTest");
}

FText FAutomatedPerfTestLaunchExtension::GetDisplayName() const
{
	return LOCTEXT("ExtensionName", "Automated Perf Test");
}

void FAutomatedPerfTestLaunchExtension::GetExtensionsMenuEntry(FExtensionsMenuEntry& MenuEntry) const
{
	MenuEntry = FExtensionsMenuEntry::OwnSection;
}

#undef LOCTEXT_NAMESPACE


