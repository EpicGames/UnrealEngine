// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Extension/AutomatedTestLaunchExtension.h"
#include "Model/ProjectLauncherModel.h"
#include "Templates/UnrealTemplate.h"

enum class EAutomatedPerfTestType : uint8
{
	Sequence,
	Replay,
	ProfileGo,
	StaticCamera,
	Material,
	MAX
};

class FAutomatedPerfTestLaunchExtensionInstance : public ProjectLauncher::FAutomatedTestLaunchExtensionInstance
{
	using Super = ProjectLauncher::FAutomatedTestLaunchExtensionInstance;
public:
	FAutomatedPerfTestLaunchExtensionInstance(FArgs& InArgs) : FAutomatedTestLaunchExtensionInstance(InArgs) {};
	virtual ~FAutomatedPerfTestLaunchExtensionInstance() = default;

	virtual void OnPropertyChanged() override;
	virtual void OnTestAdded(ILauncherProfileAutomatedTestRef AutomatedTest) override;

	virtual void CustomizeTree(ProjectLauncher::FLaunchProfileTreeData& ProfileTreeData) override;
	virtual void CustomizeUATCommandLine(FString& InOutCommandLine) override;

private:
	void ExportProfileGoScenarios(const FString& Filename);

	void AddTestNodeOptions(ProjectLauncher::FLaunchProfileTreeNode& TreeNode);
	void AddSequenceTestNodeOptions(ProjectLauncher::FLaunchProfileTreeNode& TreeNode);
	void AddReplayTestNodeOptions(ProjectLauncher::FLaunchProfileTreeNode& TreeNode);
	void AddProfileGoTestNodeOptions(ProjectLauncher::FLaunchProfileTreeNode& TreeNode);
	void AddStaticCameraTestNodeOptions(ProjectLauncher::FLaunchProfileTreeNode& TreeNode);
	void AddMaterialTestNodeOptions(ProjectLauncher::FLaunchProfileTreeNode& TreeNode);

	void OnTestTypeSelectionChanged(EAutomatedPerfTestType Type);
	void SetTestType(EAutomatedPerfTestType Type);
	EAutomatedPerfTestType GetTestType() const;

	const ProjectLauncher::FProjectSettings GetProjectSettings() const;
	const FConfigSection* GetConfigSection(EAutomatedPerfTestType TestType) const;

	template<EAutomatedPerfTestType TestType>
	ProjectLauncher::FLaunchProfileTreeNode::FCallbacks GetTestTypeCallbacks()
	{
		const auto IsTestType = [this]()
		{
			return GetTestType() == TestType;
		};

		ProjectLauncher::FLaunchProfileTreeNode::FCallbacks Callbacks
		{
			.IsVisible = IsTestType,
			.IsEnabled = IsTestType
		};

		return MoveTemp(Callbacks);
	}

	EAutomatedPerfTestType CurrentTestType = EAutomatedPerfTestType::Sequence;
};


class FAutomatedPerfTestLaunchExtension : public ProjectLauncher::FAutomatedTestLaunchExtension
{
public:
	virtual TSharedPtr<ProjectLauncher::FLaunchExtensionInstance> CreateInstanceForProfile(ProjectLauncher::FLaunchExtensionInstance::FArgs& InArgs) override;
	virtual const TCHAR* GetInternalName() const override;
	virtual FText GetDisplayName() const override;
	virtual void GetExtensionsMenuEntry(FExtensionsMenuEntry& MenuEntry) const override;
};

