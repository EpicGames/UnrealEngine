// Copyright Epic Games, Inc. All Rights Reserved.

#include "AutomatedPerfTestLaunchExtensionModule.h"

void FAutomatedPerfTestLaunchExtensionModule::StartupModule() 
{
	Extensions.Add(MakeShared<FAutomatedPerfTestLaunchExtension>());
	for (TSharedRef<ProjectLauncher::FLaunchExtension> Extension : Extensions)
	{
		IProjectLauncherModule::Get().RegisterExtension(Extension);
	}
}

void FAutomatedPerfTestLaunchExtensionModule::ShutdownModule() 
{
	if (IProjectLauncherModule* ProjectLauncher = IProjectLauncherModule::TryGet())
	{
		for (TSharedRef<ProjectLauncher::FLaunchExtension> Extension : Extensions)
		{
			ProjectLauncher->UnregisterExtension(Extension);
		}
	}

	Extensions.Reset();
}

IMPLEMENT_MODULE(FAutomatedPerfTestLaunchExtensionModule, AutomatedPerfTestLaunchExtension);

