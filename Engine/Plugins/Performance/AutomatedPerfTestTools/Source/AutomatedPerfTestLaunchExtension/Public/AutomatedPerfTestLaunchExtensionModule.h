// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"
#include "ProjectLauncherModule.h"
#include "AutomatedPerfTestLaunchExtension.h"

class FAutomatedPerfTestLaunchExtensionModule : public IModuleInterface
{
public:
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

private:
	TArray<TSharedRef<ProjectLauncher::FLaunchExtension>> Extensions;
};



